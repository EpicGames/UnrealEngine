// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"

struct FTimespan;

class FWebMAudioBackendNull
{
public:

	enum class EStreamState
	{
		NewMovie,
		ResumeAfterPause
	};

	bool InitializePlatform() { return true;  }
	void ShutdownPlatform() {}
	bool StartStreaming(int32 SampleRate, int32 NumOfChannels, EStreamState StreamState) { return true; }
	void StopStreaming() {}
	bool SendAudio(const FTimespan& Timespan, const uint8* Buffer, size_t BufferSize) { return true; }

	void Pause(bool bPause) {}
	bool IsPaused() const { return false; }
	void Tick(float DeltaTime) {}
};
