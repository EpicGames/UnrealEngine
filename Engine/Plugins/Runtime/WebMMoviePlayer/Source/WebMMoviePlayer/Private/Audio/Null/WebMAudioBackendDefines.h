// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FWebMAudioBackendNull;

using FWebMAudioBackend = FWebMAudioBackendNull;
