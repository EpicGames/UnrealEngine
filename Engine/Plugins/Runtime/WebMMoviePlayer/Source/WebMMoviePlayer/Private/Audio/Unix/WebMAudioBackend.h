// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"

THIRD_PARTY_INCLUDES_START
#include <SDL3/SDL_audio.h>
THIRD_PARTY_INCLUDES_END

class FString;
struct FTimespan;


class FWebMAudioBackendSDL
{
public:

	enum class EStreamState
	{
		NewMovie,
		ResumeAfterPause
	};

	FWebMAudioBackendSDL();
	virtual ~FWebMAudioBackendSDL();

	//~ IWebMAudioBackend interface
	virtual bool InitializePlatform();
	virtual void ShutdownPlatform();
	virtual bool StartStreaming(int32 SampleRate, int32 NumOfChannels, EStreamState StreamState);
	virtual void StopStreaming();
	virtual bool SendAudio(const FTimespan& Timespan, const uint8* Buffer, size_t BufferSize);

	virtual void Pause(bool bPause);
	virtual bool IsPaused() const;
	virtual void Tick(float DeltaTime);
	virtual FString GetDefaultDeviceName();

protected:
	SDL_AudioStream* AudioStream;
	bool bSDLInitialized;

	/** Whether we're pausing playing back sound. */
	bool bPaused;
};
