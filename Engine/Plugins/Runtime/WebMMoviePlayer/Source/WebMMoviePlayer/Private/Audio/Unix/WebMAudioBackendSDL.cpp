// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMAudioBackend.h"
#include "WebMMovieCommon.h"
#include <SDL3/SDL.h>

FWebMAudioBackendSDL::FWebMAudioBackendSDL()
	: AudioStream(nullptr)
	, bSDLInitialized(false)
	, bPaused(false)
{
}

FWebMAudioBackendSDL::~FWebMAudioBackendSDL()
{
	ShutdownPlatform();
}

bool FWebMAudioBackendSDL::InitializePlatform()
{
	int32 Result = SDL_InitSubSystem(SDL_INIT_AUDIO);
	if (Result < 0)
	{
		UE_LOGF(LogWebMMoviePlayer, Error, "SDL_InitSubSystem create failed: %d", Result);
		bSDLInitialized = false;
		return false;
	}
	else
	{
		bSDLInitialized = true;
		return true;
	}
}

void FWebMAudioBackendSDL::ShutdownPlatform()
{
	StopStreaming();

	if (bSDLInitialized)
	{
		// this is refcounted
		SDL_QuitSubSystem(SDL_INIT_AUDIO);
		bSDLInitialized = false;
	}
}

bool FWebMAudioBackendSDL::StartStreaming(int32 SampleRate, int32 NumOfChannels, EStreamState StreamState)
{
	if (!bSDLInitialized)
	{
		return false;
	}

	SDL_AudioSpec AudioSpec = {SDL_AUDIO_S16LE, NumOfChannels, SampleRate };

	AudioStream = SDL_OpenAudioDeviceStream(SDL_AUDIO_DEVICE_DEFAULT_PLAYBACK, &AudioSpec, nullptr, nullptr);

	if (!AudioStream)
	{
		UE_LOGF(LogWebMMoviePlayer, Error, "SDL_OpenAudioDevice failed");
		return false;
	}
	else
	{
		return true;
	}
}

void FWebMAudioBackendSDL::StopStreaming()
{
	if (AudioStream)
	{
		SDL_CloseAudioDevice(SDL_GetAudioStreamDevice(AudioStream));
		AudioStream = nullptr;
	}
}

bool FWebMAudioBackendSDL::SendAudio(const FTimespan& Timespan, const uint8* Buffer, size_t BufferSize)
{

	int32 Result = SDL_PutAudioStreamData(AudioStream, Buffer, BufferSize);
	if (Result < 0)
	{
		UE_LOGF(LogWebMMoviePlayer, Error, "SDL_QueueAudio failed: %d", Result);
		return false;
	}
	else
	{
		if (!bPaused)
		{
			SDL_ResumeAudioDevice(SDL_GetAudioStreamDevice(AudioStream));
		}
		return true;
	}
}

void FWebMAudioBackendSDL::Pause(bool bPause)
{
	bPaused = bPause;
	if(bPause)
		SDL_PauseAudioDevice(SDL_GetAudioStreamDevice(AudioStream));
	else
		SDL_ResumeAudioDevice(SDL_GetAudioStreamDevice(AudioStream));
}

bool FWebMAudioBackendSDL::IsPaused() const
{
	return bPaused;
}

void FWebMAudioBackendSDL::Tick(float DeltaTime)
{
}

FString FWebMAudioBackendSDL::GetDefaultDeviceName()
{
	return {};
}
