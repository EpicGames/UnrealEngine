// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMMovieStreamer.h"

#include "Modules/ModuleManager.h"

#if WITH_WEBM_LIBS && !PLATFORM_WINDOWS && !PLATFORM_MAC
	#define WITH_WEBM_STARTUP_MOVIES 1
#else
	#define WITH_WEBM_STARTUP_MOVIES 0
#endif

#if WITH_WEBM_STARTUP_MOVIES
TSharedPtr<FWebMMovieStreamer, ESPMode::ThreadSafe> WebMMovieStreamer;
#endif // WITH_WEBM_STARTUP_MOVIES

class FWebMMoviePlayerModule : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
#if WITH_WEBM_STARTUP_MOVIES
		FWebMMovieStreamer *Streamer = new FWebMMovieStreamer;
		WebMMovieStreamer = MakeShareable(Streamer);
		GetMoviePlayer()->RegisterMovieStreamer(WebMMovieStreamer);
#endif // WITH_WEBM_STARTUP_MOVIES
	}

	virtual void ShutdownModule() override
	{
#if WITH_WEBM_STARTUP_MOVIES
		WebMMovieStreamer.Reset();
#endif // WITH_WEBM_STARTUP_MOVIES
	}
};

IMPLEMENT_MODULE(FWebMMoviePlayerModule, WebMMoviePlayer)
