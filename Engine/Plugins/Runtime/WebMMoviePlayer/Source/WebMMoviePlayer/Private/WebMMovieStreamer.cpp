// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMMovieStreamer.h"
#include "WebMAudioBackend.h"
#include "WebMMovieCommon.h"

#if WITH_WEBM_LIBS
#include "MediaSamples.h"
#include "Misc/Paths.h"
#include "WebMVideoDecoder.h"
#include "WebMAudioDecoder.h"
#include "WebMContainer.h"
#include "WebMMediaAudioSample.h"
#include "WebMMediaTextureSample.h"
#endif // WITH_WEBM_LIBS

DEFINE_LOG_CATEGORY(LogWebMMoviePlayer);

#if WITH_WEBM_LIBS
FWebMMovieStreamer::FWebMMovieStreamer()
	: AudioBackend(MakeUnique<FWebMAudioBackend>())
	, Viewport(MakeShareable(new FMovieViewport()))
	, VideoFramesCurrentlyProcessing(0)
	, StartTime(0)
	, bPlaying(false)
	, bPaused(false)
	, TicksLeftToWaitPostCompletion(0)
{
}

FWebMMovieStreamer::~FWebMMovieStreamer()
{
	Cleanup();
}

void FWebMMovieStreamer::Cleanup()
{
	bPlaying = false;
	bPaused = false;
	VideoFramesCurrentlyProcessing = 0;
	TicksLeftToWaitPostCompletion = 0;

	// remove delegates if registered
	if (ResumeHandle.IsValid())
	{
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(ResumeHandle);
		ResumeHandle.Reset();
	}

	if (PauseHandle.IsValid())
	{
		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(PauseHandle);
		PauseHandle.Reset();
	}

	ReleaseAcquiredResources();
	AudioBackend->ShutdownPlatform();
}

FTextureRHIRef FWebMMovieStreamer::GetTexture()
{
	return SlateVideoTexture.IsValid() ? SlateVideoTexture->GetRHIRef() : nullptr;
}

bool FWebMMovieStreamer::Init(const TArray<FString>& InMoviePaths, TEnumAsByte<EMoviePlaybackType> InPlaybackType)
{
	// Initializes the streamer for audio and video playback of the given path(s).
	// NOTE: If multiple paths are provided, it is expect that they be played back seamlessly.
	AudioBackend->InitializePlatform();

	// Add the given paths to the movie queue
	MovieQueue.Append(InMoviePaths);

	// register delegate if not registered
	if (!ResumeHandle.IsValid())
	{
		ResumeHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FWebMMovieStreamer::HandleApplicationHasEnteredForeground);
	}
	if (!PauseHandle.IsValid())
	{
		PauseHandle = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FWebMMovieStreamer::HandleApplicationWillEnterBackground);
	}

	// start our first movie playing
	return StartNextMovie();
}

bool FWebMMovieStreamer::StartNextMovie()
{
	checkf(!bPaused, TEXT("Should not start next movie when paused!"));

	if (MovieQueue.Num() > 0)
	{
		ReleaseAcquiredResources();

		MovieName = MovieQueue[0];

		MovieQueue.RemoveAt(0);

		FString MoviePath = FPaths::ProjectContentDir() + TEXT("Movies/") + MovieName + TEXT(".webm");

		if (FPaths::FileExists(MoviePath))
		{
			Container.Reset(new FWebMContainer());
		}
		else
		{
			UE_LOGF(LogWebMMoviePlayer, Error, "Movie '%ls' not found.", *MoviePath);

			MovieName = FString();
			return false;
		}

		UE_LOGF(LogWebMMoviePlayer, Log, "Starting '%ls'", *MoviePath);

		if (!Container->Open(MoviePath))
		{
			MovieName = FString();
			return false;
		}

		Samples = MakeShareable(new FMediaSamples());
		AudioDecoder.Reset(new FWebMAudioDecoder(*this));
		VideoDecoder.Reset(new FWebMVideoDecoder(*this));

		FWebMAudioTrackInfo DefaultAudioTrack = Container->GetCurrentAudioTrackInfo();
		check(DefaultAudioTrack.bIsValid);

		FWebMVideoTrackInfo DefaultVideoTrack = Container->GetCurrentVideoTrackInfo();
		check(DefaultVideoTrack.bIsValid);

		AudioDecoder->Initialize(DefaultAudioTrack.CodecName, DefaultAudioTrack.SampleRate, DefaultAudioTrack.NumOfChannels, DefaultAudioTrack.CodecPrivateData, DefaultAudioTrack.CodecPrivateDataSize);
		VideoDecoder->Initialize(DefaultVideoTrack.CodecName);

		AudioBackend->StartStreaming(DefaultAudioTrack.SampleRate, DefaultAudioTrack.NumOfChannels, FWebMAudioBackend::EStreamState::NewMovie);

		StartTime = FPlatformTime::Seconds();
		bPlaying = true;

		return true;
	}
	else
	{
		UE_LOGF(LogWebMMoviePlayer, Log, "No Movie to start.");
		return false;
	}
}

FString FWebMMovieStreamer::GetMovieName()
{
	return MovieName;
}

bool FWebMMovieStreamer::IsLastMovieInPlaylist()
{
	return MovieQueue.Num() == 0;
}

bool FWebMMovieStreamer::Tick(FRHICommandListBase& RHICmdList, float InDeltaTime)
{
	AudioBackend->Tick(InDeltaTime);

	if (LIKELY(bPlaying && !bPaused))
	{
		if (TicksLeftToWaitPostCompletion)
		{
			if (--TicksLeftToWaitPostCompletion <= 0)
			{
				TicksLeftToWaitPostCompletion = 0;
				return !StartNextMovie();
			}

			return false;
		}

		bool bHaveThingsToDo = false;

		bHaveThingsToDo |= DisplayFrames(InDeltaTime);
		bHaveThingsToDo |= SendAudio(InDeltaTime);

		bHaveThingsToDo |= ReadMoreFrames();

		if (!bHaveThingsToDo)
		{
			// we're done playing this movie, make sure we can safely remove the textures next frame
			TicksLeftToWaitPostCompletion = 1;
			Viewport->SetTexture(nullptr);
		}
		return false;
	}
	else if (bPaused)
	{
		// continue ticking
		return false;
	}
	else
	{
		// We're done playing
		return true;
	}
}

TSharedPtr<class ISlateViewport> FWebMMovieStreamer::GetViewportInterface()
{
	return Viewport;
}

float FWebMMovieStreamer::GetAspectRatio() const
{
	return static_cast<float>(Viewport->GetSize().X) / static_cast<float>(Viewport->GetSize().Y);
}

void FWebMMovieStreamer::ForceCompletion()
{
	if (bPlaying)
	{
		bPlaying = false;
	}

	MovieQueue.Empty();
}

void FWebMMovieStreamer::ReleaseAcquiredResources()
{
	VideoDecoder.Reset();
	AudioDecoder.Reset();
	Samples.Reset();
	Container.Reset();

	SlateVideoTexture.Reset();
	Viewport->SetTexture(nullptr);

	AudioBackend->StopStreaming();
}

bool FWebMMovieStreamer::DisplayFrames(float InDeltaTime)
{
	if (!SlateVideoTexture.IsValid())
	{
		SlateVideoTexture = MakeShareable(new FSlateTexture2DRHIRef(nullptr, 0, 0));
	}

	double CurrentTime = FPlatformTime::Seconds();
	double MovieTime = CurrentTime - StartTime;

	TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> VideoSample;
	TRange<FTimespan> TimeRange(FTimespan::Zero(), FTimespan::FromSeconds(MovieTime));
	bool bFoundSample = Samples->FetchVideo(TimeRange, VideoSample);

	if (bFoundSample && VideoSample)
	{
		VideoFramesCurrentlyProcessing--;

		FWebMMediaTextureSample* WebMSample = StaticCast<FWebMMediaTextureSample*>(VideoSample.Get());

		if (SlateVideoTexture->IsValid())
		{
			SlateVideoTexture->ReleaseRHI();
		}

		SlateVideoTexture->SetRHIRef(WebMSample->GetTextureRef(), WebMSample->GetDim().X, WebMSample->GetDim().Y);

		Viewport->SetTexture(SlateVideoTexture);
	}

	return Samples->NumVideoSamples() > 0 || VideoDecoder->IsBusy();
}

bool FWebMMovieStreamer::SendAudio(float InDeltaTime)
{
	// Just send all available audio for processing

	TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe> AudioSample;
	TRange<FTimespan> TimeRange(FTimespan::Zero(), FTimespan::MaxValue());
	bool bFoundSample = Samples->FetchAudio(TimeRange, AudioSample);

	while (bFoundSample && AudioSample)
	{
		FWebMMediaAudioSample* WebMSample = StaticCast<FWebMMediaAudioSample*>(AudioSample.Get());
		AudioBackend->SendAudio(WebMSample->GetTime().Time, WebMSample->GetDataBuffer().GetData(), WebMSample->GetDataBuffer().Num());

		bFoundSample = Samples->FetchAudio(TimeRange, AudioSample);
	}

	return Samples->NumAudioSamples() > 0 || AudioDecoder->IsBusy();
}

bool FWebMMovieStreamer::ReadMoreFrames()
{
	FTimespan ReadBufferLength = FTimespan::FromSeconds(1.0 / 30.0);

	TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>> AudioFrames;
	TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>> VideoFrames;

	Container->ReadFrames(ReadBufferLength, AudioFrames, VideoFrames);

	VideoFramesToDecodeLater.Enqueue(VideoFrames);

	// Trigger video decoding
	while (!VideoFramesToDecodeLater.IsEmpty() && VideoFramesCurrentlyProcessing < 5)
	{
		if (VideoFramesToDecodeLater.Dequeue(VideoFrames))
		{
			VideoFramesCurrentlyProcessing += VideoFrames.Num();
			VideoDecoder->DecodeVideoFramesAsync(VideoFrames);
		}
	}

	// Trigger audio decoding
	if (AudioFrames.Num() > 0)
	{
		AudioDecoder->DecodeAudioFramesAsync(AudioFrames);
	}

	return VideoFrames.Num() > 0 || AudioFrames.Num() > 0;
}

void FWebMMovieStreamer::AddVideoSampleFromDecodingThread(TSharedRef<FWebMMediaTextureSample, ESPMode::ThreadSafe> Sample)
{
	Samples->AddVideo(Sample);
}

void FWebMMovieStreamer::AddAudioSampleFromDecodingThread(TSharedRef<FWebMMediaAudioSample, ESPMode::ThreadSafe> Sample)
{
	Samples->AddAudio(Sample);
}

void FWebMMovieStreamer::HandleApplicationWillEnterBackground()
{
	UE_LOGF(LogWebMMoviePlayer, Log, "FWebMMovieStreamer::HandleApplicationWillEnterBackground, pausing - we were %ls", bPaused ? TEXT("paused") : TEXT("not paused"));

	if (!bPaused)
	{
		TimeAtEnteringBackground = FPlatformTime::Seconds();
		AudioBackend->Pause(true);
		bPaused = true;
	}
}

void FWebMMovieStreamer::HandleApplicationHasEnteredForeground()
{
	UE_LOGF(LogWebMMoviePlayer, Log, "FWebMMovieStreamer::HandleApplicationHasEnteredForeground, unpausing - we were %ls", bPaused ? TEXT("paused") : TEXT("not paused"));

	if (bPaused)
	{
		double PrevStartTimeForLog = StartTime;
		double TimeSpentInBackground = FPlatformTime::Seconds() - TimeAtEnteringBackground;
		StartTime += TimeSpentInBackground;
		AudioBackend->Pause(false);
		bPaused = false;
		UE_LOGF(LogWebMMoviePlayer, Log, "Spent %f seconds in background, adjusting start time from %f to %f", TimeSpentInBackground, PrevStartTimeForLog, StartTime);
	}
}

#endif // WITH_WEBM_LIBS
