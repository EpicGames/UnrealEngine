// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WEBM_LIBS

#include "MoviePlayer.h"

#include COMPILED_PLATFORM_HEADER(WebMAudioBackendDefines.h)
#include "Containers/Queue.h"
#include "WebMSamplesSink.h"

class FSlateTexture2DRHIRef;
class FWebMVideoDecoder;
class FWebMAudioDecoder;
class FMediaSamples;
class FWebMContainer;
struct FWebMFrame;

class FWebMMovieStreamer : public IMovieStreamer, public IWebMSamplesSink
{
public:
	FWebMMovieStreamer();
	~FWebMMovieStreamer();

	//~ IWebMSamplesSink interface
	virtual void AddVideoSampleFromDecodingThread(TSharedRef<FWebMMediaTextureSample, ESPMode::ThreadSafe> Sample) override;
	virtual void AddAudioSampleFromDecodingThread(TSharedRef<FWebMMediaAudioSample, ESPMode::ThreadSafe> Sample) override;
	virtual void ReportVideoDecodingError(FString InErrorMessage) override { }
	virtual void ReportAudioDecodingError(FString InErrorMessage) override { }

	//~ IMediaOutput interface
	virtual bool Init(const TArray<FString>& InMoviePaths, TEnumAsByte<EMoviePlaybackType> InPlaybackType) override;
	virtual void ForceCompletion() override;
	virtual bool Tick(FRHICommandListBase& RHICmdList, float InDeltaTime) override;
	virtual TSharedPtr<class ISlateViewport> GetViewportInterface() override;
	virtual float GetAspectRatio() const override;
	virtual FString GetMovieName() override;
	virtual bool IsLastMovieInPlaylist() override;
	virtual void Cleanup() override;
	virtual FTextureRHIRef GetTexture() override;

	FOnCurrentMovieClipFinished OnCurrentMovieClipFinishedDelegate;
	virtual FOnCurrentMovieClipFinished& OnCurrentMovieClipFinished() override { return OnCurrentMovieClipFinishedDelegate; }

private:
	TArray<FString> MovieQueue;
	TQueue<TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>> VideoFramesToDecodeLater;
	FString MovieName;
	TUniquePtr<FWebMVideoDecoder> VideoDecoder;
	TUniquePtr<FWebMAudioDecoder> AudioDecoder;
	TUniquePtr<FWebMContainer> Container;
	TUniquePtr<FWebMAudioBackend> AudioBackend;
	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> Samples;
	TSharedPtr<FMovieViewport> Viewport;
	TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> SlateVideoTexture;

	/** Limits amount of frames being processed to save video memory used by the player */
	int32 VideoFramesCurrentlyProcessing;

	/** Movie start time */
	double StartTime;
	/** Timestamp when we entered background */
	double TimeAtEnteringBackground;
	/** Whether a movie is being played */
	bool bPlaying;
	/** We can be paused e.g. while in background */
	bool bPaused;

	/**
	 * Number of ticks to wait after the movie is complete before moving on to the next one.
	 *
	 * This allows us to defer texture deletion while it is being displayed.
	 */
	int32 TicksLeftToWaitPostCompletion;

	bool StartNextMovie();
	void ReleaseAcquiredResources();
	bool DisplayFrames(float InDeltaTime);
	bool SendAudio(float InDeltaTime);
	bool ReadMoreFrames();

	/** Callback for when the application resumed in the foreground. */
	void HandleApplicationHasEnteredForeground();

	/** Callback for when the application is being paused in the background. */
	void HandleApplicationWillEnterBackground();

	/** Foreground/background delegate for pause. */
	FDelegateHandle PauseHandle;

	/** Foreground/background delegate for resume. */
	FDelegateHandle ResumeHandle;

};

#endif // WITH_WEBM_LIBS
