// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System;
using System.IO;

public class WebMMoviePlayer : ModuleRules
{
	public WebMMoviePlayer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"MoviePlayer",
				"RenderCore",
				"RHI",
				"SlateCore",
				"Slate",
				"MediaUtils",
				"WebMMedia",
			});

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			PrivateDependencyModuleNames.Add("SDL3");

			PrivateIncludePaths.Add("WebMMoviePlayer/Private/Audio/Unix");
		}
		else
		{
			PrivateIncludePaths.Add("WebMMoviePlayer/Private/Audio/Null");
		}
		PrivateIncludePaths.Add("WebMMoviePlayer/Private/Audio");

		bool bHaveWebMlibs = 
			Target.Platform == UnrealTargetPlatform.Mac ||
			Target.Platform == UnrealTargetPlatform.Win64 ||
			(Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) && Target.Architecture == UnrealArch.X64);
		PublicDefinitions.Add("WITH_WEBM_LIBS=" + (bHaveWebMlibs ? "1" : "0"));
	}
}
