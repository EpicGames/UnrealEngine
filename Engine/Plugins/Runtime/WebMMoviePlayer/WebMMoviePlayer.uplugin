{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Movie Player for WebM files",
	"Description": "Movie Player for WebM files",
	"Category": "Media Players",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "WebMMoviePlayer",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			],
			"TargetDenyList": [
				"Server"
			]
		}
	],
	"Plugins": [
		{
			"Name": "WebMMedia",
			"Enabled": true
		}
	],
	"IsExperimentalVersion": false
}